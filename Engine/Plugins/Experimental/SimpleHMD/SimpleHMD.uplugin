{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "SimpleHMD",
	"Description" : "SimpleHMD is a sample of a basic stereo HMD implementation",
	"Category" : "Virtual Reality",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"SupportedTargetPlatforms": [
		"Win64",
		"Linux",
		"Android",
		"Mac",
		"IOS",
		"VisionOS"
	],
	"Modules" : 
	[
		{
			"Name" : "SimpleHMD",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		}
	],
	"Plugins" : 
	[
		{
			"Name" : "XRBase",
			"Enabled" : true
		}
	]
}