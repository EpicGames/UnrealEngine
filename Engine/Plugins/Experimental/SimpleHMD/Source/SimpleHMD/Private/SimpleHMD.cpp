// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimpleHMD.h"
#include "GameFramework/PlayerController.h"
#include "Engine/LocalPlayer.h"
#include "ISimpleHMDPlugin.h"
#include "PostProcess/PostProcessHMD.h"
#include "GameFramework/WorldSettings.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "SceneView.h"
#include "RHIResourceUtils.h"

//---------------------------------------------------
// SimpleHMD Plugin Implementation
//---------------------------------------------------

class FSimpleHMDPlugin : public ISimpleHMDPlugin
{
	/** IHeadMountedDisplayModule implementation */
	virtual TSharedPtr< class IXRTrackingSystem, ESPMode::ThreadSafe > CreateTrackingSystem() override;

	FString GetModuleKeyName() const override
	{
		return FString(TEXT("SimpleHMD"));
	}
};

IMPLEMENT_MODULE( FSimpleHMDPlugin, SimpleHMD )

TSharedPtr< class IXRTrackingSystem, ESPMode::ThreadSafe > FSimpleHMDPlugin::CreateTrackingSystem()
{
	auto SimpleHMD = FSceneViewExtensions::NewExtension<FSimpleHMD>();
	if( SimpleHMD->IsInitialized() )
	{
		FGenericCrashContext::SetEngineData(TEXT("XR"), TEXT("XR SimpleHMD"));
		return SimpleHMD;
	}
	return nullptr;
}


float FSimpleHMD::GetWorldToMetersScale() const
{
	return WorldToMeters;
}

//---------------------------------------------------
// SimpleHMD IHeadMountedDisplay Implementation
//---------------------------------------------------

bool FSimpleHMD::IsHMDEnabled() const
{
	return true;
}

void FSimpleHMD::EnableHMD(bool enable)
{
}

bool FSimpleHMD::GetHMDMonitorInfo(MonitorInfo& MonitorDesc)
{
	MonitorDesc.MonitorName = "";
	MonitorDesc.MonitorId = 0;
	MonitorDesc.DesktopX = MonitorDesc.DesktopY = MonitorDesc.ResolutionX = MonitorDesc.ResolutionY = 0;
	return false;
}

void FSimpleHMD::GetFieldOfView(float& OutHFOVInDegrees, float& OutVFOVInDegrees) const
{
	OutHFOVInDegrees = 0.0f;
	OutVFOVInDegrees = 0.0f;
}

bool FSimpleHMD::EnumerateTrackedDevices(TArray<int32>& OutDevices, EXRTrackedDeviceType Type)
{
	if (Type == EXRTrackedDeviceType::Any || Type == EXRTrackedDeviceType::HeadMountedDisplay)
	{
		OutDevices.Add(IXRTrackingSystem::HMDDeviceId);
		return true;
	}
	return false;
}

void FSimpleHMD::SetInterpupillaryDistance(float NewInterpupillaryDistance)
{
}

float FSimpleHMD::GetInterpupillaryDistance() const
{
	return 0.064f;
}

void FSimpleHMD::GetHMDOrientation(FQuat& CurrentOrientation)
{
	// very basic.  no head model, no prediction, using debuglocalplayer
	ULocalPlayer* Player = GEngine->GetDebugLocalPlayer();

	if (Player != NULL && Player->PlayerController != NULL)
	{
		FVector RotationRate = Player->PlayerController->GetInputVectorKeyState(EKeys::RotationRate);

		double CurrentTime = FApp::GetCurrentTime();
		double DeltaTime = 0.0;

		if (LastSensorTime >= 0.0)
		{
			DeltaTime = CurrentTime - LastSensorTime;
		}

		LastSensorTime = CurrentTime;

		// mostly incorrect, but we just want some sensor input for testing
		RotationRate *= DeltaTime;
		CurrentOrientation *= FQuat(FRotator(FMath::RadiansToDegrees(-RotationRate.X), FMath::RadiansToDegrees(-RotationRate.Y), FMath::RadiansToDegrees(-RotationRate.Z)));
	}
	else
	{
		CurrentOrientation = FQuat(FRotator(0.0f, 0.0f, 0.0f));
	}
}

bool FSimpleHMD::GetCurrentPose(int32 DeviceId, FQuat& CurrentOrientation, FVector& CurrentPosition)
{
	if (DeviceId != IXRTrackingSystem::HMDDeviceId)
	{
		return false;
	}
	CurrentOrientation = CurHmdOrientation;
	CurrentPosition = FVector(0.0f, 0.0f, 0.0f);
	GetHMDOrientation(CurrentOrientation);
	CurHmdOrientation = LastHmdOrientation = CurrentOrientation;
	return true;
}

bool FSimpleHMD::IsChromaAbCorrectionEnabled() const
{
	return false;
}

void FSimpleHMD::ResetOrientationAndPosition(float yaw)
{
	ResetOrientation(yaw);
	ResetPosition();
}

void FSimpleHMD::ResetOrientation(float Yaw)
{
}

void FSimpleHMD::ResetPosition()
{
}

void FSimpleHMD::SetBaseRotation(const FRotator& BaseRot)
{
}

FRotator FSimpleHMD::GetBaseRotation() const
{
	return FRotator::ZeroRotator;
}

void FSimpleHMD::SetBaseOrientation(const FQuat& BaseOrient)
{
}

FQuat FSimpleHMD::GetBaseOrientation() const
{
	return FQuat::Identity;
}

void FSimpleHMD::DrawDistortionMesh_RenderThread(struct FHeadMountedDisplayPassContext& Context, const FIntPoint& TextureSize)
{
	float ClipSpaceQuadZ = 0.0f;
	FMatrix QuadTexTransform = FMatrix::Identity;
	FMatrix QuadPosTransform = FMatrix::Identity;
	const FSceneView& View = Context.View;
	const FIntRect SrcRect = View.UnscaledViewRect;

	FRHICommandListImmediate& RHICmdList = Context.RHICmdList;
	const FSceneViewFamily& ViewFamily = *(View.Family);
	FIntPoint ViewportSize = ViewFamily.RenderTarget->GetSizeXY();
	RHICmdList.SetViewport(0, 0, 0.0f, ViewportSize.X, ViewportSize.Y, 1.0f);

	static const uint32 NumVerts = 4;
	static const uint32 NumTris = 2;

	static const FDistortionVertex MeshVerts[][NumVerts] = {
		{
			// left eye
			{ FVector2f(-0.9f, -0.9f), FVector2f(0.0f, 1.0f), FVector2f(0.0f, 1.0f), FVector2f(0.0f, 1.0f), 1.0f, 0.0f },
			{ FVector2f(-0.1f, -0.9f), FVector2f(0.5f, 1.0f), FVector2f(0.5f, 1.0f), FVector2f(0.5f, 1.0f), 1.0f, 0.0f },
			{ FVector2f(-0.1f, 0.9f), FVector2f(0.5f, 0.0f), FVector2f(0.5f, 0.0f), FVector2f(0.5f, 0.0f), 1.0f, 0.0f },
			{ FVector2f(-0.9f, 0.9f), FVector2f(0.0f, 0.0f), FVector2f(0.0f, 0.0f), FVector2f(0.0f, 0.0f), 1.0f, 0.0f },
		},
		{
			// right eye
			{ FVector2f(0.1f, -0.9f), FVector2f(0.5f, 1.0f), FVector2f(0.5f, 1.0f), FVector2f(0.5f, 1.0f), 1.0f, 0.0f },
			{ FVector2f(0.9f, -0.9f), FVector2f(1.0f, 1.0f), FVector2f(1.0f, 1.0f), FVector2f(1.0f, 1.0f), 1.0f, 0.0f },
			{ FVector2f(0.9f, 0.9f), FVector2f(1.0f, 0.0f), FVector2f(1.0f, 0.0f), FVector2f(1.0f, 0.0f), 1.0f, 0.0f },
			{ FVector2f(0.1f, 0.9f), FVector2f(0.5f, 0.0f), FVector2f(0.5f, 0.0f), FVector2f(0.5f, 0.0f), 1.0f, 0.0f },
		}
	};
	static const uint16 Indices[] = { 0, 1, 2, 0, 2, 3 };

	FBufferRHIRef VertexBufferRHI = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("FSimpleHMD"), EBufferUsageFlags::Volatile, MakeConstArrayView(MeshVerts[View.StereoViewIndex]));
	FBufferRHIRef IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("FSimpleHMD"), EBufferUsageFlags::Volatile, MakeConstArrayView(Indices));

	RHICmdList.SetStreamSource(0, VertexBufferRHI, 0);
	RHICmdList.DrawIndexedPrimitive(IndexBufferRHI, 0, 0, NumVerts, 0, NumTris, 1);

	IndexBufferRHI.SafeRelease();
	VertexBufferRHI.SafeRelease();
}

bool FSimpleHMD::IsStereoEnabled() const
{
	return true;
}

bool FSimpleHMD::EnableStereo(bool stereo)
{
	return true;
}

void FSimpleHMD::AdjustViewRect(int32 ViewIndex, int32& X, int32& Y, uint32& SizeX, uint32& SizeY) const
{
	SizeX = SizeX / 2;
	X += SizeX * ViewIndex;
}

void FSimpleHMD::CalculateStereoViewOffset(const int32 ViewIndex, FRotator& ViewRotation, const float InWorldToMeters, FVector& ViewLocation)
{
	if( ViewIndex != INDEX_NONE)
	{
		float EyeOffset = 3.20000005f;
		const float PassOffset = (ViewIndex == EStereoscopicEye::eSSE_LEFT_EYE) ? EyeOffset : -EyeOffset;
		ViewLocation += ViewRotation.Quaternion().RotateVector(FVector(0,PassOffset,0));
	}
}

FMatrix FSimpleHMD::GetStereoProjectionMatrix(const int32 ViewIndex) const
{
	const float ProjectionCenterOffset = 0.151976421f;
	const float PassProjectionOffset = (ViewIndex == EStereoscopicEye::eSSE_LEFT_EYE) ? ProjectionCenterOffset : -ProjectionCenterOffset;

	const float HalfFov = 2.19686294f / 2.f;
	const float InWidth = 640.f;
	const float InHeight = 480.f;
	const float XS = 1.0f / tan(HalfFov);
	const float YS = InWidth / tan(HalfFov) / InHeight;

	const float InNearZ = GNearClippingPlane;
	return FMatrix(
		FPlane(XS,                      0.0f,								    0.0f,							0.0f),
		FPlane(0.0f,					YS,	                                    0.0f,							0.0f),
		FPlane(0.0f,	                0.0f,								    0.0f,							1.0f),
		FPlane(0.0f,					0.0f,								    InNearZ,						0.0f))

		* FTranslationMatrix(FVector(PassProjectionOffset,0,0));
}

void FSimpleHMD::GetEyeRenderParams_RenderThread(const FHeadMountedDisplayPassContext& Context, FVector2D& EyeToSrcUVScaleValue, FVector2D& EyeToSrcUVOffsetValue) const
{
	EyeToSrcUVOffsetValue = FVector2D::ZeroVector;
	EyeToSrcUVScaleValue = FVector2D(1.0f, 1.0f);
}


void FSimpleHMD::SetupViewFamily(FSceneViewFamily& InViewFamily)
{
	InViewFamily.EngineShowFlags.MotionBlur = 0;
	InViewFamily.EngineShowFlags.HMDDistortion = true;
	InViewFamily.EngineShowFlags.StereoRendering = IsStereoEnabled();

	if (UWorld* World = GWorld)
	{
		WorldToMeters = World->GetWorldSettings()->WorldToMeters;
	}
}

void FSimpleHMD::SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView)
{
	InView.BaseHmdOrientation = FQuat(FRotator(0.0f,0.0f,0.0f));
	InView.BaseHmdLocation = FVector(0.f);
}

FSimpleHMD::FSimpleHMD(const FAutoRegister& AutoRegister) :
	FHeadMountedDisplayBase(nullptr),
	FHMDSceneViewExtension(AutoRegister),
	CurHmdOrientation(FQuat::Identity),
	LastHmdOrientation(FQuat::Identity),
	DeltaControlRotation(FRotator::ZeroRotator),
	DeltaControlOrientation(FQuat::Identity),
	LastSensorTime(-1.0),
	WorldToMeters(100.0f)
{
}

FSimpleHMD::~FSimpleHMD()
{
}

bool FSimpleHMD::IsInitialized() const
{
	return true;
}
