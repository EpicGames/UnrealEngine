// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HeadMountedDisplayBase.h"
#include "SceneViewExtension.h"

class APlayerController;
class FSceneView;
class FSceneViewFamily;
class UCanvas;

/**
 * Simple Head Mounted Display
 */
class FSimpleHMD : public FHeadMountedDisplayBase, public FHMDSceneViewExtension
{
public:
	/** IXRTrackingSystem interface */
	virtual FName GetSystemName() const override
	{
		static FName DefaultName(TEXT("SimpleHMD"));
		return DefaultName;
	}

	int32 GetXRSystemFlags() const
	{
		return EXRSystemFlags::IsHeadMounted;
	}

	virtual bool EnumerateTrackedDevices(TArray<int32>& OutDevices, EXRTrackedDeviceType Type = EXRTrackedDeviceType::Any) override;

	virtual void SetInterpupillaryDistance(float NewInterpupillaryDistance) override;
	virtual float GetInterpupillaryDistance() const override;

	virtual void ResetOrientationAndPosition(float yaw = 0.f) override;
	virtual void ResetOrientation(float Yaw = 0.f) override;
	virtual void ResetPosition() override;

	virtual bool GetCurrentPose(int32 DeviceId, FQuat& CurrentOrientation, FVector& CurrentPosition) override;
	virtual void SetBaseRotation(const FRotator& BaseRot) override;
	virtual FRotator GetBaseRotation() const override;

	virtual void SetBaseOrientation(const FQuat& BaseOrient) override;
	virtual FQuat GetBaseOrientation() const override;

	virtual class IHeadMountedDisplay* GetHMDDevice() override
	{
		return this;
	}

	virtual class TSharedPtr< class IStereoRendering, ESPMode::ThreadSafe > GetStereoRenderingDevice() override
	{
		return SharedThis(this);
	}

protected:
	/** FXRTrackingSystemBase protected interface */
	virtual float GetWorldToMetersScale() const override;

public:
	/** IHeadMountedDisplay interface */
	virtual bool IsHMDConnected() override { return true; }
	virtual bool IsHMDEnabled() const override;
	virtual void EnableHMD(bool allow = true) override;
	virtual bool GetHMDMonitorInfo(MonitorInfo&) override;
	virtual void GetFieldOfView(float& OutHFOVInDegrees, float& OutVFOVInDegrees) const override;
	virtual bool IsChromaAbCorrectionEnabled() const override;
	virtual void DrawDistortionMesh_RenderThread(struct FHeadMountedDisplayPassContext& Context, const FIntPoint& TextureSize) override;

	/** IStereoRendering interface */
	virtual bool IsStereoEnabled() const override;
	virtual bool EnableStereo(bool stereo = true) override;
	virtual void AdjustViewRect(int32 ViewIndex, int32& X, int32& Y, uint32& SizeX, uint32& SizeY) const override;
	virtual void CalculateStereoViewOffset(const int32 ViewIndex, FRotator& ViewRotation,
		const float InWorldToMeters, FVector& ViewLocation) override;
	virtual FMatrix GetStereoProjectionMatrix(const int32 ViewIndex) const override;
	virtual void GetEyeRenderParams_RenderThread(const struct FHeadMountedDisplayPassContext& Context, FVector2D& EyeToSrcUVScaleValue, FVector2D& EyeToSrcUVOffsetValue) const override;

	/** ISceneViewExtension interface */
	virtual void SetupViewFamily(FSceneViewFamily& InViewFamily) override;
	virtual void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView) override;

public:
	/** Constructor */
	FSimpleHMD(const FAutoRegister&);

	/** Destructor */
	virtual ~FSimpleHMD();

	/** @return	True if the HMD was initialized OK */
	bool IsInitialized() const;

private:

	FQuat					CurHmdOrientation;
	FQuat					LastHmdOrientation;

	FRotator				DeltaControlRotation;    // same as DeltaControlOrientation but as rotator
	FQuat					DeltaControlOrientation; // same as DeltaControlRotation but as quat

	double					LastSensorTime;
	float					WorldToMeters;

	void GetHMDOrientation(FQuat& CurrentOrientation);
};

