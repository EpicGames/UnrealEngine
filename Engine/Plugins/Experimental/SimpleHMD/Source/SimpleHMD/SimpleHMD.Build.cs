// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class SimpleHMD : ModuleRules
	{
		public SimpleHMD(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateIncludePaths.AddRange(
				new string[] {
				}
				);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"InputCore",
					"RHI",
					"RenderCore",
					"Renderer",
					"HeadMountedDisplay",
					"XRBase"
				}
				);
		}
	}
}
